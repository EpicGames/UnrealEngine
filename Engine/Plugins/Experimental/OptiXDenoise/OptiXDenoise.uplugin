{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "OptiXDenoise",
	"Description" : "Denoising engine for the Unreal Path Tracer based on NVIDIA's OptiX AI-Accelerated Denoiser library.",
	"Category" : "Denoising",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsExperimental" : true,
	"IsExperimentalVersion": true, 
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "OptiXDenoise",
			"Type" : "RuntimeAndProgram",
			"LoadingPhase" : "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [
				"Win64:arm64"
			]
		}
	]
}
