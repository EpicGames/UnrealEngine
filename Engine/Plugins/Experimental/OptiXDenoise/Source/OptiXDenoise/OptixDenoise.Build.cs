// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class OptiXDenoise : ModuleRules
	{
		public OptiXDenoise(ReadOnlyTargetRules Target) : base(Target)
		{

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"RenderCore",
					"Renderer",
					"RHI",
					"CUDA",
					"Projects",
					"OptiXDenoiseBase"
				}
			);
			
			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("MessageLog");
			}

			if(Target.Platform == UnrealTargetPlatform.Win64)
			{
				PrivateDependencyModuleNames.Add("D3D12RHI");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
			}
		}
	}
}
