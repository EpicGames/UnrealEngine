// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptiXDenoiser.h"
#include "RHI.h"
#include "RHIResources.h"

#if PLATFORM_WINDOWS
#include "ShaderCore.h"
#include "ID3D12DynamicRHI.h"
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <VersionHelpers.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#define LOG_CALLBACK_LEVEL 4

namespace UE::OptiXDenoiser
{

	static void ContextLogCallback(uint32_t Level, const char* Tag, const char* Message, void* /*cbdata*/)
	{
		if (Level < LOG_CALLBACK_LEVEL)
		{
			UE_LOGF(LogOptiXDenoise, Log, "[%d][%ls]: %ls", Level, *FString(ANSI_TO_TCHAR(Tag)), *FString(ANSI_TO_TCHAR(Message)));

		}
	}
	
	FOptiXImage2D::~FOptiXImage2D()
	{
		ReleaseCUDABuffer();
		DestroyCUDAIntermediateMemory();
	}

	void FOptiXImage2D::ReleaseCUDABuffer()
	{
#if PLATFORM_WINDOWS
		if (CudaBuffer.Handle)
		{
			CloseHandle(CudaBuffer.Handle);
			CudaBuffer.Handle = nullptr;
		}
#endif
		if (CudaBuffer.SurfaceObject && CudaBuffer.DestroyFunc)
		{
			CudaBuffer.DestroyFunc(CudaBuffer.SurfaceObject);
			CudaBuffer.SurfaceObject = 0;
		}

		CudaBuffer.DestroyFunc = nullptr;
		CudaBuffer.CopyCallback = nullptr;
	}
	void FOptiXImage2D::DestroyCUDAIntermediateMemory()
	{
		if (CudaBuffer.CudaPtr)
		{
			FCUDAModule::CUDA().cuCtxPushCurrent(FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext());
			CUDA_CHECK(FCUDAModule::CUDA().cuMemFree((CUdeviceptr)CudaBuffer.CudaPtr));
			FCUDAModule::CUDA().cuCtxPopCurrent(NULL);

			CudaBuffer.CudaPtr = nullptr;
			CudaBuffer.CudaMemorySize = 0;
		}
	}

	FOptiXImageData& FOptiXImage2D::GetRawImage()
	{
		check(Format == EOptiXImageFormat::CUDA_A32B32G32R32_F || Format == EOptiXImageFormat::CUDA_FloatRGBA);

		uint32_t PixelSize = Format == EOptiXImageFormat::CUDA_A32B32G32R32_F ? (4 * static_cast<uint32_t>(sizeof(float))) : (4 * static_cast<uint32_t>(sizeof(uint16_t)));
		uint32_t RowStrideInBytes = PixelSize * Width;

		Image.Data = (CUdeviceptr)CudaBuffer.CudaPtr;
		Image.Width = Width;
		Image.Height = Height;
		Image.RowStrideInBytes = RowStrideInBytes;
		Image.PixelStrideInBytes = PixelSize;
		Image.Format = Format;
		
		return Image;
	}

	void FOptiXImage2D::SetTexture(TSurfaceObject InTexture, EUnderlyingRHI UnderlyingRHI, void* SharedHandle,
		FReleaseCUDATextureCallback InOnReleaseCUDATextureCallback, FCUDASurfaceTextureCopyCallback InCopyCallback)
	{
		if (Format == EOptiXImageFormat::CUDA_A32B32G32R32_F || Format == EOptiXImageFormat::CUDA_FloatRGBA)
		{
			check(CudaBuffer.Handle == nullptr);
			check(CudaBuffer.SurfaceObject == 0);

			CudaBuffer.UnderlyingRHI = UnderlyingRHI;
			CudaBuffer.Handle = SharedHandle;
			CudaBuffer.SurfaceObject = InTexture;
			CudaBuffer.DestroyFunc = InOnReleaseCUDATextureCallback;
			CudaBuffer.CopyCallback = InCopyCallback;

			// Allocate the linear memory to CudaPtr to copy between the surface object if necessary on the device.
			if (CudaBuffer.SurfaceObject)
			{
				const size_t NewDeviceMemorySize = (Format == EOptiXImageFormat::CUDA_A32B32G32R32_F ? sizeof(float) : sizeof(uint16_t)) * 4 * Width * Height;

				if (CudaBuffer.CudaPtr && CudaBuffer.CudaMemorySize != NewDeviceMemorySize)
				{
					DestroyCUDAIntermediateMemory();
				}

				if (!CudaBuffer.CudaPtr)
				{
					FCUDAModule::CUDA().cuCtxPushCurrent(FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext());
					CUDA_CHECK(FCUDAModule::CUDA().cuMemAlloc(reinterpret_cast<CUdeviceptr*>(&CudaBuffer.CudaPtr), NewDeviceMemorySize));
					FCUDAModule::CUDA().cuCtxPopCurrent(NULL);

					CudaBuffer.CudaMemorySize = NewDeviceMemorySize;
				} 
			}
			else
			{
				UE_LOGF(LogOptiXDenoise, Warning, "SetTexture | Cuda surface object is null");
			}
		}
	}

	void FOptiXImage2D::Flush(ECUDASurfaceCopyType CopyType, CUstream CudaStream)
	{
		check(CudaBuffer.CopyCallback);
		if (CudaBuffer.CopyCallback)
		{
			check(CudaBuffer.SurfaceObject);
			check(CudaBuffer.CudaPtr);

			CudaBuffer.CopyCallback(CudaBuffer.SurfaceObject, *this, CudaStream, CopyType);
		}
	}

	FOptiXImageFactory::FOptiXImageFactory()
	{
	}

	FOptiXImageFactory::~FOptiXImageFactory()
	{
		FlushImages();
	}

	void FOptiXImageFactory::FlushImages()
	{
		TextureToOptiXImage2DMapping.Empty();
	}

	void FOptiXImageFactory::RemoveStaleTextures()
	{
		TMap<FTextureRHIRef, TSharedPtr<FOptiXImage2D>>::TIterator Iter = TextureToOptiXImage2DMapping.CreateIterator();
		for (; Iter; ++Iter)
		{
			FTextureRHIRef& Tex = Iter.Key();

			if (Tex.GetRefCount() == 1)
			{
				Iter.RemoveCurrent();
			}
		}
	}

	TSharedPtr<FOptiXImage2D> FOptiXImageFactory::GetOrCreateImage(const FTextureRHIRef InTexture)
	{
		check(InTexture.IsValid());

		RemoveStaleTextures();

		TSharedPtr<FOptiXImage2D> OutImage;
		
		if (TextureToOptiXImage2DMapping.Contains(InTexture))
		{
			OutImage = *(TextureToOptiXImage2DMapping.Find(InTexture));
			OutImage->SetWidth(InTexture->GetDesc().Extent.X);
			OutImage->SetHeight(InTexture->GetDesc().Extent.Y);
		}
		else
		{
			TSharedPtr<FOptiXImage2D> NewImage = MakeShareable(new FOptiXImage2D());
		
			EPixelFormat Format = InTexture->GetFormat();
			NewImage->SetWidth(InTexture->GetDesc().Extent.X);
			NewImage->SetHeight(InTexture->GetDesc().Extent.Y);

			// Setup 
			switch (Format)
			{
			case EPixelFormat::PF_A32B32G32R32F:
				NewImage->SetFormat(EOptiXImageFormat::CUDA_A32B32G32R32_F);
				break;
			case EPixelFormat::PF_FloatRGBA:
				NewImage->SetFormat(EOptiXImageFormat::CUDA_FloatRGBA);
				break;
			default:
				check(false);
				break;
			}

			OutImage = MoveTemp(NewImage);

			SetTexture(OutImage, InTexture);
			TextureToOptiXImage2DMapping.Add(InTexture, OutImage);
		}

		return OutImage;
	}

	TSharedPtr<FOptiXImage2D> FOptiXImageFactory::GetOptiXImage2DAndSetTexture(FTextureRHIRef InTexture)
	{
		TSharedPtr<FOptiXImage2D> Image = GetOrCreateImage(InTexture);
		return Image;
	}

	void FOptiXImageFactory::SetTexture(TSharedPtr<FOptiXImage2D> FOptiXImageData, const FTextureRHIRef& Texture)
	{
		const ERHIInterfaceType RHIType = RHIGetInterfaceType();

		if (RHIType == ERHIInterfaceType::Vulkan)
		{
			SetTextureCUDAVulkan(FOptiXImageData, Texture);
		}
#if PLATFORM_WINDOWS
		else if (RHIType == ERHIInterfaceType::D3D11)
		{
			UE_LOGF(LogOptiXDenoise, Error, "OptiX Denoiser does not support D3D11. Please use other RHI.");
		}
		else if (RHIType == ERHIInterfaceType::D3D12)
		{
			if (IsRHIDeviceNVIDIA())
			{
				this->SetTextureCUDAD3D12(FOptiXImageData, Texture);
			}
			else
			{
				UE_LOGF(LogOptiXDenoise, Error, "OptiX Denoiser is only supported on NVIDIA device. Current RHI - %ls", GDynamicRHI->GetName());
			}
		}
#endif
		else
		{
			UE_LOGF(LogOptiXDenoise, Error, "OptiX Denoiser is only supported on NVIDIA device. Current RHI - %ls", GDynamicRHI->GetName());
		}
	}

	void FOptiXImageFactory::SetTextureCUDAVulkan(TSharedPtr<FOptiXImage2D> FOptiXImageData, const FTextureRHIRef& Texture)
	{
		UE_LOGF(LogOptiXDenoise, Error, "OptiX Denoiser is not supported on Vulkan Yet.");
	}

#if PLATFORM_WINDOWS
	void FOptiXImageFactory::SetTextureCUDAD3D12(TSharedPtr<FOptiXImage2D> FOptiXImageData, const FTextureRHIRef& Texture)
	{
		ID3D12Resource* NativeD3D12Resource = GetID3D12DynamicRHI()->RHIGetResource(Texture);
		const int64 TextureMemorySize = GetID3D12DynamicRHI()->RHIGetResourceMemorySize(Texture);

		// The resource needs to be committed, otherwise the interop fails.
		// To prevent a mystery crash in future, check that our resource is a committed resource
		check(!GetID3D12DynamicRHI()->RHIIsResourcePlaced(Texture));

		TRefCountPtr<ID3D12Device> OwnerDevice;
		HRESULT QueryResult;
		if ((QueryResult = NativeD3D12Resource->GetDevice(IID_PPV_ARGS(OwnerDevice.GetInitReference()))) != S_OK)
		{
			UE_LOGF(LogOptiXDenoise, Error, "Failed to get DX texture handle for importing memory to CUDA: %d (Get Device)", QueryResult);
		}

		//
		// ID3D12Device::CreateSharedHandle gives as an NT Handle, and so we need to call CloseHandle on it
		//
		HANDLE D3D12TextureHandle;
		if ((QueryResult = OwnerDevice->CreateSharedHandle(NativeD3D12Resource, NULL, GENERIC_ALL, NULL, &D3D12TextureHandle)) != S_OK)
		{
			UE_LOGF(LogOptiXDenoise, Error, "Failed to get DX texture handle for importing memory to CUDA: %d", QueryResult);
		}

		FCUDAModule::CUDA().cuCtxPushCurrent(FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext());

		CUexternalMemory MappedExternalMemory = nullptr;

		{
			// generate a cudaExternalMemoryHandleDesc
			CUDA_EXTERNAL_MEMORY_HANDLE_DESC CudaExtMemHandleDesc = {};
			CudaExtMemHandleDesc.type = CU_EXTERNAL_MEMORY_HANDLE_TYPE_D3D12_RESOURCE;
			CudaExtMemHandleDesc.handle.win32.name = NULL;
			CudaExtMemHandleDesc.handle.win32.handle = D3D12TextureHandle;
			CudaExtMemHandleDesc.size = TextureMemorySize;
			// Necessary for committed resources (DX11 and committed DX12 resources)
			CudaExtMemHandleDesc.flags |= CUDA_EXTERNAL_MEMORY_DEDICATED;

			// import external memory
			CUresult Result = FCUDAModule::CUDA().cuImportExternalMemory(&MappedExternalMemory, &CudaExtMemHandleDesc);
			if (Result != CUDA_SUCCESS)
			{
				UE_LOGF(LogOptiXDenoise, Error, "Failed to import external memory error: %d", Result);
			}
		}

		CUmipmappedArray MappedMipArray = nullptr;
		CUarray MappedArray = nullptr;

		{
			CUDA_EXTERNAL_MEMORY_MIPMAPPED_ARRAY_DESC MipmapDesc = {};
			MipmapDesc.numLevels = 1;
			MipmapDesc.offset = 0;
			MipmapDesc.arrayDesc.Width = Texture->GetDesc().Extent.X;
			MipmapDesc.arrayDesc.Height = Texture->GetDesc().Extent.Y;
			MipmapDesc.arrayDesc.Depth = 1;
			MipmapDesc.arrayDesc.NumChannels = 4;
			switch (Texture->GetDesc().Format)
			{
				case PF_A32B32G32R32F: MipmapDesc.arrayDesc.Format = CU_AD_FORMAT_FLOAT; break;
				case PF_FloatRGBA:     MipmapDesc.arrayDesc.Format = CU_AD_FORMAT_HALF; break;
				default: check(false);
			}
			MipmapDesc.arrayDesc.Flags = CUDA_ARRAY3D_SURFACE_LDST;

			// get the CUarray from the external memory
			CUresult Result = FCUDAModule::CUDA().cuExternalMemoryGetMappedMipmappedArray(&MappedMipArray, MappedExternalMemory, &MipmapDesc);
			if (Result != CUDA_SUCCESS)
			{
				UE_LOGF(LogOptiXDenoise, Error, "Failed to bind mipmappedArray error: %d", Result);
			}
		}

		// Get the CUarray from the external memory
		CUresult MipMapArrGetLevelErr = FCUDAModule::CUDA().cuMipmappedArrayGetLevel(&MappedArray, MappedMipArray, 0);
		if (MipMapArrGetLevelErr != CUDA_SUCCESS)
		{
			UE_LOGF(LogOptiXDenoise, Error, "Failed to bind to mip 0.");
		}

		// Bind to surface object
		CUsurfObject CuSurfaceObject;
		{
			CUDA_RESOURCE_DESC CudaResourceDesc = {};
			CudaResourceDesc.resType = CU_RESOURCE_TYPE_ARRAY;
			CudaResourceDesc.res.array.hArray = MappedArray;

			CUresult Result = FCUDAModule::CUDA().cuSurfObjectCreate(&CuSurfaceObject, &CudaResourceDesc);
			if (Result != CUDA_SUCCESS)
			{
				UE_LOGF(LogOptiXDenoise, Error, "Failed to create surface object error: %d", Result);
			}
		}

		FCUDAModule::CUDA().cuCtxPopCurrent(NULL);

		FOptiXImageData->SetTexture(CuSurfaceObject, FOptiXImage2D::EUnderlyingRHI::D3D12, D3D12TextureHandle,
			// Destroy Callback
			[MappedArray, MappedMipArray, MappedExternalMemory](TSurfaceObject CuSurfaceObject) {
			// free the cuda types
			FCUDAModule::CUDA().cuCtxPushCurrent(FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext());

			if (CuSurfaceObject)
			{
				CUresult Result = FCUDAModule::CUDA().cuSurfObjectDestroy(CuSurfaceObject);

				if (Result != CUDA_SUCCESS)
				{
					UE_LOGF(LogOptiXDenoise, Error, "Failed to destroy Surface Object: %d", Result);
				}
			}

			if (MappedArray)
			{
				CUresult Result = FCUDAModule::CUDA().cuArrayDestroy(MappedArray);
				if (Result != CUDA_SUCCESS)
				{
					UE_LOGF(LogOptiXDenoise, Error, "Failed to destroy MappedArray: %d", Result);
				}
			}

			if (MappedMipArray)
			{
				CUresult Result = FCUDAModule::CUDA().cuMipmappedArrayDestroy(MappedMipArray);
				if (Result != CUDA_SUCCESS)
				{
					UE_LOGF(LogOptiXDenoise, Error, "Failed to destroy MappedMipArray: %d", Result);
				}
			}

			if (MappedExternalMemory)
			{
				CUresult Result = FCUDAModule::CUDA().cuDestroyExternalMemory(MappedExternalMemory);
				if (Result != CUDA_SUCCESS)
				{
					UE_LOGF(LogOptiXDenoise, Error, "Failed to destroy MappedExternalMemoryArray: %d", Result);
				}
			}

			FCUDAModule::CUDA().cuCtxPopCurrent(NULL);
			},
			
			// Copy callback
			[this](TSurfaceObject SurfaceObject, FOptiXImage2D& Image, CUstream Stream, ECUDASurfaceCopyType CopyType) {

				check(SurfaceObject);

				if (CopyType == ECUDASurfaceCopyType::SurfaceToBuffer)
				{
					FOptiXCudaFunctionList::Get().CopySurfaceToCudaBuffer(SurfaceObject, Image.GetRawImage(), Stream);
				}
				else if (CopyType == ECUDASurfaceCopyType::BufferToSurface)
				{
					FOptiXCudaFunctionList::Get().CopyCudaBufferToSurface(SurfaceObject, Image.GetRawImage(), Stream);
				}
			});
	}
#endif

	FOptiXFlowEstimator::~FOptiXFlowEstimator()
	{
		Destroy();
	}

	void FOptiXFlowEstimator::Init(
		uint32_t InWidth,
		uint32_t InHeight,
		EOptiXImageFormat InPixelFormat /*= OPTIX_PIXEL_FORMAT_FLOAT4*/)
	{
		checkf(InWidth > 0 && InHeight > 0, TEXT("Image size should be larger than 0"));

		// the context does not change, we should reuse the instance
		if (Width == InWidth && Height == InHeight && PixelFormat == InPixelFormat && OptixFlow)
		{
			return;
		}
		else
		{
			Destroy();
		}

		Width = InWidth;
		Height = InHeight;
		PixelFormat = InPixelFormat;

		// @todo add more pixel format support
		check(PixelFormat == EOptiXImageFormat::CUDA_A32B32G32R32_F);

		// Initialize the context and the optical flow
		CUcontext CUContext = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext();
			FCUDAModule::CUDA().cuCtxPushCurrent(CUContext);

		{
			CUDA_CHECK(FCUDAModule::CUDA().cuStreamCreate(&CudaStream, CU_STREAM_DEFAULT));
			OptixFlow = MakeUnique<FOpticalFlowContext>();
			OPTIX_CHECK(OptixFlow->Init(CUContext, CudaStream, Width, Height));
		}

		FCUDAModule::CUDA().cuCtxPopCurrent(NULL);
	}

	void FOptiXFlowEstimator::Commit()
	{
		Update();
		Execute();
	}

	void FOptiXFlowEstimator::Update()
	{
	}

	void FOptiXFlowEstimator::Execute()
	{
		check(OptixFlow);

		uint32_t FlowImageIndex = static_cast<uint32_t>(EOpticalFlowImageType::FLOWOUTPUT);

		FOptiXImageData Flow = GetOptiXImage2D(EOpticalFlowImageType::FLOWOUTPUT)->GetRawImage();
		
		const FOptiXImageData InputRefernceArray[2] = {
			GetOptiXImage2D(EOpticalFlowImageType::INPUTFRAME)->GetRawImage(),
			GetOptiXImage2D(EOpticalFlowImageType::REFERENCE)->GetRawImage(),
		};

		CUDA_CHECK(FCUDAModule::CUDA().cuCtxPushCurrent(FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext()));
		OPTIX_CHECK(OptixFlow->ComputeFlow(Flow, (const FOptiXImageData*)&InputRefernceArray[0]));
		FCUDAModule::CUDA().cuCtxPopCurrent(NULL);

	}

	void FOptiXFlowEstimator::Destroy()
	{
		if (OptixFlow)
		{
			CUDA_CHECK(FCUDAModule::CUDA().cuCtxPushCurrent(FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext()));
			OptixFlow->Destroy();
			OptixFlow.Reset();
			FCUDAModule::CUDA().cuCtxPopCurrent(NULL);
		}
	}

	FOptiXGPUDenoiser::~FOptiXGPUDenoiser()
	{
		Finish();
	}

	void FOptiXGPUDenoiser::SetOptiXImage2D(EDenoisingImageType Type, TSharedPtr<FOptiXImage2D> Image)
	{ 
		Images[static_cast<uint32_t>(Type)] = Image; 
	}

	void FOptiXGPUDenoiser::Init(
		uint32_t InWidth,
		uint32_t InHeight,
		EOptiXImageFormat InPixelFormat /*= OPTIX_PIXEL_FORMAT_FLOAT4*/,
		uint32_t InTileWidth /*= 0u*/,
		uint32_t InTileHeight /*= 0u*/,
		bool InGuideAlbedo/* = true*/,
		bool InGuideNormal/* = true*/,
		bool KPMode /*= false*/,
		bool TemporalMode /*= true*/)
	{
		GuideAlbedo = InGuideAlbedo;
		GuideNormal = InGuideNormal;

		checkf(!GuideNormal || GuideAlbedo,
			TEXT("Albedo is required when normal is given"));
		check((InTileHeight == 0 && InTileWidth == 0) || (InTileHeight > 0 && InTileWidth > 0));
		
		InternalDenoisedFrameId = 0;

		DenoiseDimension = TemporalMode ? EDenoiseDimension::SPATIAL_TEMPORAL : EDenoiseDimension::SPATIAL;
		TileHeight = InTileHeight > 0 ? InTileHeight : InHeight;
		TileWidth = InTileWidth > 0 ? InTileWidth : InWidth;
		Width = InWidth;
		Height = InHeight;


		PixelFormat = InPixelFormat;

		CUcontext CUContext = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext();
		CUDA_CHECK(FCUDAModule::CUDA().cuCtxPushCurrent(CUContext));

		// Initialize CUDA and OptiX context
		{
			CUDA_CHECK(FCUDAModule::CUDA().cuMemFree((CUdeviceptr)nullptr));
			CUDA_CHECK(FCUDAModule::CUDA().cuStreamCreate(&CudaStream, CU_STREAM_DEFAULT));

			OptiXDenoiseContext = MakeUnique<FOptiXDenoiseContext>();

			OPTIX_CHECK(OptiXDenoiseContext->InitOptiX());
			OPTIX_CHECK(OptiXDenoiseContext->CreateContext(CUContext, &ContextLogCallback, LOG_CALLBACK_LEVEL));
		}

		OPTIX_CHECK(OptiXDenoiseContext->InitializeDenoiser(GuideNormal, GuideAlbedo, TemporalMode));

		// Setup denoiser
		{
			OPTIX_CHECK(OptiXDenoiseContext->ComputeMemoryResource(TileWidth,TileHeight));

			if (InTileWidth == 0)
			{
				CudaScratchSize = static_cast<uint32_t>(OptiXDenoiseContext->GetWithoutOverlapScratchSizeInBytes());
				CudaOverlap = 0;
			}
			else
			{
				CudaScratchSize = static_cast<uint32_t>(OptiXDenoiseContext->GetWithOverlapScratchSizeInBytes());
				CudaOverlap = static_cast<uint32_t>(OptiXDenoiseContext->GetOverlapWindowSizeInPixels());
			}
			CUDA_CHECK(FCUDAModule::CUDA().cuMemAlloc(reinterpret_cast<CUdeviceptr*>(&CudaScratch), CudaScratchSize));

			CudaStateSize = static_cast<uint32_t>(OptiXDenoiseContext->GetStateSizeInBytes());
			CUDA_CHECK(FCUDAModule::CUDA().cuMemAlloc(reinterpret_cast<CUdeviceptr*>(&CudaState), CudaStateSize));
			
			check(KPMode == false);
			CUDA_CHECK(FCUDAModule::CUDA().cuMemAlloc(reinterpret_cast<CUdeviceptr*>(&Intensity), sizeof(float)));
			CUDA_CHECK(FCUDAModule::CUDA().cuMemAlloc(reinterpret_cast<CUdeviceptr*>(&AverageColor), 3 * sizeof(float)));
			
			// Memory allocation for internal use when temporal AOV mode is enabled
			// Initialize the memory in to zero for the first frame
			#if 0
			if (false /*DenoiseDimension == EDenoiseDimension::SPATIAL_TEMPORAL && ModelKind == OPTIX_DENOISER_MODEL_KIND_TEMPORAL_AOV*/)
			{
				size_t InternalSize = Width * Height * OptiXDenoiseContext->GetInternalGuideLayerPixelSizeInBytes();
				CUDA_CHECK(FCUDAModule::CUDA().cuMemAlloc(&InternalMemIn, InternalSize));
				CUDA_CHECK(FCUDAModule::CUDA().cuMemAlloc(&InternalMemOut, InternalSize));
				CUDA_CHECK(FCUDAModule::CUDA().cuMemsetD8(InternalMemIn, 0, InternalSize));

				FOptiXImageData PreviousOutputInternalGuideLayer;

				PreviousOutputInternalGuideLayer.Data = InternalMemIn;
				PreviousOutputInternalGuideLayer.Width = Width;
				PreviousOutputInternalGuideLayer.Height = Height;
				PreviousOutputInternalGuideLayer.PixelStrideInBytes = unsigned(OptiXDenoiseContext->GetInternalGuideLayerPixelSizeInBytes());
				PreviousOutputInternalGuideLayer.RowStrideInBytes =
					PreviousOutputInternalGuideLayer.Width * PreviousOutputInternalGuideLayer.PixelStrideInBytes;
				PreviousOutputInternalGuideLayer.Format = EOptiXImageFormat::INTERNAL_LAYER;

				OptiXDenoiseContext->SetPreviousOutputInternalGuideLayer(PreviousOutputInternalGuideLayer);

				FOptiXImageData OutputInternalGuideLayer = PreviousOutputInternalGuideLayer;
				OutputInternalGuideLayer.Data = InternalMemOut;
				OptiXDenoiseContext->SetOutputInternalGuideLayer(OutputInternalGuideLayer);
			}
			#endif

			OPTIX_CHECK(OptiXDenoiseContext->SetupDenoiser(
				CudaStream,
				TileWidth + 2 * CudaOverlap,
				TileHeight + 2 * CudaOverlap,
				CudaState,
				CudaStateSize,
				CudaScratch,
				CudaScratchSize));

			// Need to denoise the alpha channel ?
			(*OptiXDenoiseContext)
				.SetDenoiseAlpha(EOptiXDenoiseAlphaMode::COPY)
				.SetHdrIntensity(Intensity)
				.SetHdrAverageColor(AverageColor)
				.SetBlendFactor(0.0f)
				.SetTemporalModeUsePreviousLayers(false);

			//Params.denoiseAlpha = OPTIX_DENOISER_ALPHA_MODE_COPY;
			//Params.hdrIntensity = Intensity;
			//Params.hdrAverageColor = AverageColor;
			//Params.blendFactor = 0.0f;
			//Params.temporalModeUsePreviousLayers = 0;
		}

		CUDA_CHECK(FCUDAModule::CUDA().cuCtxPopCurrent(NULL));
	}

	void FOptiXGPUDenoiser::Update()
	{
		if (DenoiseDimension == EDenoiseDimension::SPATIAL_TEMPORAL)
		{
			check(InternalDenoisedFrameId == 0 || (InternalDenoisedFrameId > 0 && GetOptiXImage2D(EDenoisingImageType::FLOW)));
		}

		const bool bIsFirstFrame = InternalDenoisedFrameId == 0;
		
		//Update Layer
		{
			//Layer = {};
			OptiXDenoiseContext->SetLayerInput(GetOptiXImage2D(EDenoisingImageType::COLOR)->GetRawImage());
			OptiXDenoiseContext->SetLayerOutput(GetOptiXImage2D(EDenoisingImageType::OUTPUT)->GetRawImage());


			if (DenoiseDimension == EDenoiseDimension::SPATIAL_TEMPORAL)
			{
				// Use the input as the history of the previous frame if it's the first frame
				if (bIsFirstFrame)
				{
					OptiXDenoiseContext->SetLayerPreviousOutput(GetOptiXImage2D(EDenoisingImageType::COLOR)->GetRawImage());
				}
				else
				{
					OptiXDenoiseContext->SetLayerPreviousOutput(GetOptiXImage2D(EDenoisingImageType::PREVOUTPUT)->GetRawImage());
				}
			}
		}

		// Update GuideLayer
		{
			if (GuideAlbedo)
			{
				OptiXDenoiseContext->SetGuideLayerAlbedo(GetOptiXImage2D(EDenoisingImageType::ALBEDO)->GetRawImage());
			}

			if (GuideNormal)
			{
				OptiXDenoiseContext->SetGuideLayerNormal(GetOptiXImage2D(EDenoisingImageType::NORMAL)->GetRawImage());
			}

			if (DenoiseDimension == EDenoiseDimension::SPATIAL_TEMPORAL)
			{
				OptiXDenoiseContext->SetGuideLayerFlow(GetOptiXImage2D(EDenoisingImageType::FLOW)->GetRawImage());

			}

			if (DenoiseDimension == EDenoiseDimension::SPATIAL_TEMPORAL
				&& InternalDenoisedFrameId > 0 && false /*@todo Add AOV denoising*/
				)
			{
				/*FOptiXImageData temp = GuideLayer.previousOutputInternalGuideLayer;
				GuideLayer.previousOutputInternalGuideLayer = GuideLayer.outputInternalGuideLayer;
				GuideLayer.outputInternalGuideLayer = temp;*/
			}
		}

	}

	void FOptiXGPUDenoiser::Execute()
	{
		// Optix denoiser requires to set a Cuda context as well.
		CUDA_CHECK(FCUDAModule::CUDA().cuCtxPushCurrent(FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext()));

		// Assure the first frame's flow image is black
		//Params.temporalModeUsePreviousLayers = (InternalDenoisedFrameId == 0) ? 0 : 1;
		OptiXDenoiseContext->SetTemporalModeUsePreviousLayers(InternalDenoisedFrameId != 0);

		if (Intensity)
		{
			OPTIX_CHECK(OptiXDenoiseContext->ComputeIntensity(
				CudaStream,
				Intensity,
				CudaScratch,
				CudaScratchSize));
		}

		if (AverageColor)
		{
			OPTIX_CHECK(OptiXDenoiseContext->ComputeAverageColor(
				CudaStream,
				AverageColor,
				CudaScratch,
				CudaScratchSize
			));
		}

		if (CudaOverlap == 0)
		{
			OPTIX_CHECK(OptiXDenoiseContext->InvokeOptiXDenoise(
				CudaStream,
				CudaState,
				CudaStateSize,
				0,	// input offset x
				0,	// input offset y
				CudaScratch,
				CudaScratchSize));
		}
		else
		{
			OPTIX_CHECK(OptiXDenoiseContext->InvokeOptiXDenoise(
				CudaStream,
				CudaState,
				CudaStateSize,
				CudaScratch,
				CudaScratchSize,
				CudaOverlap,
				TileWidth,
				TileHeight
			));
		}

		CUDA_CHECK(FCUDAModule::CUDA().cuCtxPopCurrent(NULL));

		InternalDenoisedFrameId++;
	}

	void FOptiXGPUDenoiser::Commit()
	{
		Update();
		Execute();
	}

	void FOptiXGPUDenoiser::Finish()
	{
		if (OptiXDenoiseContext)
		{
			OptiXDenoiseContext->Destroy();
		}

		FCUDAModule& CudaModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");
		if (CudaModule.IsAvailable())
		{
			FCUDAModule::CUDA().cuCtxPushCurrent(CudaModule.GetCudaContext());

			CUDA_CHECK(FCUDAModule::CUDA().cuMemFree(Intensity));
			CUDA_CHECK(FCUDAModule::CUDA().cuMemFree(AverageColor));
			CUDA_CHECK(FCUDAModule::CUDA().cuMemFree(CudaScratch));
			CUDA_CHECK(FCUDAModule::CUDA().cuMemFree(CudaState));
			CUDA_CHECK(FCUDAModule::CUDA().cuMemFree(InternalMemIn));
			CUDA_CHECK(FCUDAModule::CUDA().cuMemFree(InternalMemOut));

			FCUDAModule::CUDA().cuCtxPopCurrent(NULL);
		}
	}
}
