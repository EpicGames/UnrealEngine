// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//-------------------------------------------------------------------------------------------------
// OptixDenoiser wrapper using cuda driver API instead of runtime library
// Since OptiX Image requires linear memory and RHI texture does not have a linear memory mapping
// A copy is performed in CUDA for generalized API support. The data flow is as below
// Communication between UE texture and OptiX texture:
//	FRHITexture <----(DX/CUDA interop)----> CudaSurface <---(device copy)---> OptiX Image


#include "CoreMinimal.h"
#include "CudaModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogOptiXDenoise, Log, All);

#define OPTIX_CHECK(Call)																		\
	do																							\
	{																							\
		auto Result = Call;																\
		if(Result != 0)																\
		{																						\
			UE_LOGF(LogOptiXDenoise, Error, "Optix call (%ls) failed with code [%d] (%ls: %d", ANSI_TO_TCHAR(#Call), Result, ANSI_TO_TCHAR(__FILE__), (__LINE__));	\
		}																						\
	} while (false);																			\

#define CUDA_CHECK(Call)																		\
	do																							\
	{																							\
		auto Error = Call;																		\
		if (Error != 0)																			\
		{																						\
			UE_LOGF(LogOptiXDenoise, Error, "Cuda call (%ls) failed with code [%d] (%ls: %d", ANSI_TO_TCHAR(#Call), Error, ANSI_TO_TCHAR(__FILE__), (__LINE__));	\
		}																						\
	} while (false);																			\

#include "OptiXDenoiserFunctionList.h"
#include "OptiXDenoiseBase.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
#include "Windows/WindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "RHI.h"
#include "RHIResources.h"

namespace UE::OptiXDenoiser
{
	enum class ECUDASurfaceCopyType
	{
		SurfaceToBuffer,
		BufferToSurface
	};

	class FOptiXImage2D
	{
	public:

		enum class EUnderlyingRHI
		{
			Undefined,
			D3D12,
			Vulkan
		};

		using FReleaseCUDATextureCallback = TFunction<void(TSurfaceObject)>;
		using FCUDASurfaceTextureCopyCallback = TFunction<void(TSurfaceObject, FOptiXImage2D&, CUstream, ECUDASurfaceCopyType)>;
		
		// Struct to hold buffer for cuda inter-operation.
		struct FCudaBuffer
		{
			EUnderlyingRHI UnderlyingRHI;

			// @todo add support for vulkan, now support only windows
			HANDLE Handle = NULL;

			// Mapped to external texture object
			TSurfaceObject SurfaceObject = 0;

			// cudaExternalMemory_t ExternalMemory;
			void* CudaPtr = nullptr;

			// Device memory size of CudaPtr;
			unsigned int CudaMemorySize = 0;

			FReleaseCUDATextureCallback DestroyFunc;

			FCUDASurfaceTextureCopyCallback CopyCallback;
		};


		bool IsValid() { return  CudaBuffer.CudaPtr && CudaBuffer.SurfaceObject; }
		FOptiXImageData& GetRawImage();
		void SetTexture(TSurfaceObject InTexture, EUnderlyingRHI UnderlyingRHI, void* SharedHandle, 
			FReleaseCUDATextureCallback InOnReleaseTexture,
			FCUDASurfaceTextureCopyCallback InCopyCallback
			);
		void Flush(ECUDASurfaceCopyType CopyType, CUstream CudaStream);
		void SetFormat(EOptiXImageFormat InFormat) { Format = InFormat; }
		const FCudaBuffer& GetCudaBuffer() const { return CudaBuffer; }
		FCudaBuffer& GetCudaBuffer() { return CudaBuffer; }
		void SetWidth(uint32 InWidth) { Width = InWidth; }
		uint32 GetWidth() const { return Width; }
		void SetHeight(uint32 InHeight) { Height = InHeight; }
		uint32 GetHeight() const { return Height; }

		~FOptiXImage2D();
		FOptiXImage2D() {}
	protected:

		void DestroyCUDAIntermediateMemory();
		void ReleaseCUDABuffer();

		FCudaBuffer					CudaBuffer = {};
		FOptiXImageData				Image = {};
		uint32						Width = 0;
		uint32						Height = 0;
		EOptiXImageFormat			Format = EOptiXImageFormat::Undefined;
	};

	class FOptiXImageFactory
	{
	public:
		FOptiXImageFactory();
		~FOptiXImageFactory();

		TSharedPtr<FOptiXImage2D> GetOptiXImage2DAndSetTexture(FTextureRHIRef InTexture);
		void FlushImages();
	private:
		TSharedPtr<FOptiXImage2D> GetOrCreateImage(const FTextureRHIRef InTexture);

		void SetTexture(TSharedPtr<FOptiXImage2D> FOptiXImageData, const FTextureRHIRef& Texture);

		void SetTextureCUDAVulkan(TSharedPtr<FOptiXImage2D> FOptiXImageData, const FTextureRHIRef& Texture);
#if PLATFORM_WINDOWS
		void SetTextureCUDAD3D12(TSharedPtr<FOptiXImage2D> FOptiXImageData, const FTextureRHIRef& Texture);
#endif

		void RemoveStaleTextures();

	private:
		uint64 FrameId = 0;
		TMap <TRefCountPtr<FRHITexture>, TSharedPtr<FOptiXImage2D>> TextureToOptiXImage2DMapping;
	};


	enum class EOpticalFlowImageType : uint32_t
	{
		INPUTFRAME = 0,
		REFERENCE,
		FLOWOUTPUT,
		MAX
	};

	
	// Estimate the flow based on the optiX OpticalFlow
	// from source to target.
	// viable to newer GPUs.
	class FOptiXFlowEstimator
	{
	public:
		struct FData
		{
			uint32_t Width = 0;
			uint32_t Height = 0;

			HANDLE	SharedInputHandle = nullptr;
			HANDLE	SharedReferenceHandle = nullptr;

			HANDLE	SharedOutputHandle = nullptr;

			unsigned long long Size = 0;
		};

		CUstream GetCudaStream() { return CudaStream; }

		void Init(
			uint32_t InWidth,
			uint32_t InHeight,
			EOptiXImageFormat InPixelFormat = EOptiXImageFormat::CUDA_A32B32G32R32_F);
		void SetOptiXImage2D(EOpticalFlowImageType Type, TSharedPtr<FOptiXImage2D> Image) { Images[static_cast<uint32_t>(Type)] = Image; }
		TSharedPtr<FOptiXImage2D> GetOptiXImage2D(EOpticalFlowImageType Type) { return Images[static_cast<uint32_t>(Type)]; }

		void Commit();
		~FOptiXFlowEstimator();
	private:

		void Update();
		void Execute();
		void Destroy();

		TSharedPtr<FOptiXImage2D>	Images[static_cast<uint32_t>(EOpticalFlowImageType::MAX)];

		CUstream CudaStream = nullptr;

		EOptiXImageFormat PixelFormat;
		uint32_t Width;
		uint32_t Height;
		uint32_t PixelSize;

	private:
		TUniquePtr<FOpticalFlowContext> OptixFlow = nullptr;
	};

	enum class EDenoiseDimension
	{
		SPATIAL = 0,
		SPATIAL_TEMPORAL = 1
	};

	enum class EDenoisingImageType : uint32_t
	{
		COLOR = 0,
		ALBEDO,
		NORMAL,
		FLOW,
		OUTPUT,
		PREVOUTPUT,
		MAX
	};

	// Denoise the rendering with/without flow
	class FOptiXGPUDenoiser
	{
	public:
		FOptiXGPUDenoiser() {}
		FOptiXGPUDenoiser(const FOptiXGPUDenoiser&) = delete;
		~FOptiXGPUDenoiser();

		void Init(
			uint32_t Width,
			uint32_t Height,
			EOptiXImageFormat InPixelFormat = EOptiXImageFormat::CUDA_A32B32G32R32_F,
			uint32_t TileWidth = 0u,
			uint32_t TileHeight = 0u,
			bool GuildAlbedo = true,
			bool GuildNormal = true,
			bool KPMode = false,
			bool InTemporalMode = true);
		void Finish();
		void Commit();
		void SetOptiXImage2D(EDenoisingImageType Type, TSharedPtr<FOptiXImage2D> Image);
		TSharedPtr<FOptiXImage2D> GetOptiXImage2D(EDenoisingImageType Type) { return Images[static_cast<uint32_t>(Type)];}
		CUstream GetCudaStream() const { return CudaStream; }
		bool IsImageSizeChanged(uint32_t InWidth, uint32_t InHeight) { return InWidth != Width || InHeight != Height; }

	private:
		
		void Update();
		void Execute();

		EDenoiseDimension DenoiseDimension;
		bool GuideAlbedo;
		bool GuideNormal;

		uint32_t TileWidth;
		uint32_t TileHeight;
		uint32_t Width;
		uint32_t Height;
		uint32_t PixelSize;

		EOptiXImageFormat PixelFormat;

		// Cuda stream for nonblocking
		CUstream CudaStream;

		CUdeviceptr	CudaState = 0;
		uint32_t	CudaStateSize = 0;

		CUdeviceptr CudaScratch = 0;
		uint32_t	CudaScratchSize = 0;
		uint32_t	CudaOverlap = 0;

		CUdeviceptr Intensity = 0;
		CUdeviceptr AverageColor = 0;
		CUdeviceptr InternalMemIn = 0;
		CUdeviceptr InternalMemOut = 0;

		TSharedPtr<FOptiXImage2D> Images[static_cast<uint32_t>(EDenoisingImageType::MAX)] = {};

		uint32_t InternalDenoisedFrameId = 0;

	private:

		// The wrapper entry for NVIDIA's OptiX Denoiser 
		TUniquePtr<FOptiXDenoiseContext> OptiXDenoiseContext;
	};

}
