// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptiXDenoiser.h"
#include "HAL/IConsoleManager.h"
#include "PathTracingDenoiser.h"
#include "RenderGraphBuilder.h"
#include "SceneView.h"
#include <mutex>

#define LOCTEXT_NAMESPACE "FOptiXDenoiseModule"

DEFINE_LOG_CATEGORY(LogOptiXDenoise);

BEGIN_SHADER_PARAMETER_STRUCT(FDenoiseTextureExtParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(InputAlbedo, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(InputNormal, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(InputFlow, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(InputPreviousOutput, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMotionVectorParameters, )
	RDG_TEXTURE_ACCESS(InputFrameTexture, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(ReferenceFrameTexture, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

namespace
{
	TAutoConsoleVariable<int32> CVarOptixDenoiseTileWidth(
		TEXT("r.OptixDenoise.TileWidth"),
		256,
		TEXT("Set the tile width for denosing. 0 to use the image size\n"),
		ECVF_RenderThreadSafe
	);
	
	TAutoConsoleVariable<int32> CVarOptixDenoiseTileHeight(
		TEXT("r.OptixDenoise.TileHeight"),
		256,
		TEXT("Set the tile height for denosing\n"),
		ECVF_RenderThreadSafe
	);

}

using namespace UE::OptiXDenoiser;

class FOptiXDenoiseModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	static TUniquePtr<FOptiXGPUDenoiser> Denoiser;
	static TUniquePtr<FOptiXFlowEstimator> FlowEstimator;
	static TUniquePtr<FOptiXImageFactory> OptiXImageFactory;

	void* OptiXDenoiseBaseDLLHandle;
};

TUniquePtr<FOptiXGPUDenoiser> FOptiXDenoiseModule::Denoiser = MakeUnique<FOptiXGPUDenoiser>();
TUniquePtr<FOptiXFlowEstimator> FOptiXDenoiseModule::FlowEstimator = MakeUnique<FOptiXFlowEstimator>();
TUniquePtr<FOptiXImageFactory> FOptiXDenoiseModule::OptiXImageFactory = MakeUnique<FOptiXImageFactory>();

static void OpticalFlow(
	FRHICommandListImmediate& RHICmdList,
	FRHITexture* InputTex,// previous frame
	FRHITexture* ReferenceTex,// current frame
	FRHITexture* FlowOutputTex, // empty flow texture to write to
	float PreExposure,
	FRHIGPUMask GPUMask
	)
{
	FOptiXCudaFunctionList::Get().InitializeCudaModule();

	FIntPoint Size = InputTex->GetSizeXY();

	const CUstream& CudaStream = FOptiXDenoiseModule::FlowEstimator->GetCudaStream();
	{
		auto UploadTextureToDenoiser = [&CudaStream, 
			&ImageFactory = FOptiXDenoiseModule::OptiXImageFactory, 
			&FlowEstimator = FOptiXDenoiseModule::FlowEstimator,
			PreExposure]
		(FRHITexture* RHITexture, EOpticalFlowImageType ImageType, const TCHAR* Name)
		{
			if (RHITexture)
			{
				auto OptiXImage = ImageFactory->GetOptiXImage2DAndSetTexture(RHITexture);
				
				if (ImageType != EOpticalFlowImageType::FLOWOUTPUT)
				{
					FOptiXCudaFunctionList::Get().
						CopySurfaceToCudaBuffer(
							OptiXImage->GetCudaBuffer().SurfaceObject,
							OptiXImage->GetRawImage(),
							CudaStream,
							false,
							PreExposure);
				}

				FlowEstimator->SetOptiXImage2D(ImageType, OptiXImage);
			}
			else
			{
				FlowEstimator->SetOptiXImage2D(ImageType, nullptr);
			}
		};

		UploadTextureToDenoiser(InputTex, EOpticalFlowImageType::INPUTFRAME, TEXT("InputFrameTex"));
		UploadTextureToDenoiser(ReferenceTex, EOpticalFlowImageType::REFERENCE, TEXT("ReferenceTex"));
		UploadTextureToDenoiser(FlowOutputTex, EOpticalFlowImageType::FLOWOUTPUT, TEXT("FlowOutputTex"));
	}

	FOptiXDenoiseModule::FlowEstimator->Init(Size.X, Size.Y);
	FOptiXDenoiseModule::FlowEstimator->Commit();

	auto OutputImage = FOptiXDenoiseModule::FlowEstimator->GetOptiXImage2D(EOpticalFlowImageType::FLOWOUTPUT);
	OutputImage->Flush(ECUDASurfaceCopyType::BufferToSurface, CudaStream);

	CUDA_CHECK(FCUDAModule::CUDA().cuCtxPushCurrent(FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext()));
	CUDA_CHECK(FCUDAModule::CUDA().cuStreamSynchronize(CudaStream));
	CUDA_CHECK(FCUDAModule::CUDA().cuCtxPopCurrent(NULL));

	FOptiXDenoiseModule::OptiXImageFactory->FlushImages();

	UE_LOGF(LogOptiXDenoise, Log, "Estimate Optical Flow (%d x %d)", Size.X, Size.Y);
}

static void Denoise(
	FRHICommandListImmediate& RHICmdList,
	FRHITexture* ColorTex,
	FRHITexture* AlbedoTex,
	FRHITexture* NormalTex,
	FRHITexture* OutputTex,
	FRHITexture* FlowTex,
	FRHITexture* PreviousOutputTex,
	int DenoisingFrameId,
	bool bForceSpatialDenoiserOnly,
	FRHIGPUMask GPUMask)
{
	FOptiXCudaFunctionList::Get().InitializeCudaModule();

	FIntPoint Size = ColorTex->GetSizeXY();

	static std::once_flag CallOnceFlag;
	std::call_once(CallOnceFlag, []() {
			CUcontext CudaContext = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext();
			FCUDAModule::CUDA().cuCtxPushCurrent(CudaContext);
			unsigned int CUDAAPIVersion = 0;
			FCUDAModule::CUDA().cuCtxGetApiVersion(CudaContext, &CUDAAPIVersion);

			FCUDAModule::CUDA().cuCtxPopCurrent(NULL);
			UE_LOGF(LogOptiXDenoise, Log, "CUDA API Version:%d", CUDAAPIVersion);
		});

	// (Re)initialize the denoiser if this is the first frame (e.g., new cut) or the image size has changed
	if (bForceSpatialDenoiserOnly ||
		FOptiXDenoiseModule::Denoiser->IsImageSizeChanged(Size.X, Size.Y))
	{
		FOptiXDenoiseModule::Denoiser->Finish();
		FOptiXDenoiseModule::Denoiser->Init(
			Size.X,
			Size.Y,
			EOptiXImageFormat::CUDA_A32B32G32R32_F,
			CVarOptixDenoiseTileWidth.GetValueOnAnyThread(),
			CVarOptixDenoiseTileHeight.GetValueOnAnyThread());

		UE_LOGF(LogOptiXDenoise, Log, "Denoising Frame %d (%d x %d) Spatially",
			DenoisingFrameId, Size.X, Size.Y);
	}
	else
	{
		UE_LOGF(LogOptiXDenoise, Log, "Denoising Frame %d (%d x %d) temporally",
			DenoisingFrameId, Size.X, Size.Y);
	}

	const CUstream& CudaStream = FOptiXDenoiseModule::Denoiser->GetCudaStream();
	{
		auto UploadTextureToDenoiser = [&CudaStream, &ImageFactory = FOptiXDenoiseModule::OptiXImageFactory, &Denoiser = FOptiXDenoiseModule::Denoiser]
		(FRHITexture* RHITexture, EDenoisingImageType ImageType, const TCHAR* Name)
		{
			if (RHITexture)
			{
				auto OptiXImage = ImageFactory->GetOptiXImage2DAndSetTexture(RHITexture);
				
				if (ImageType == EDenoisingImageType::NORMAL)
				{
					const bool bIsNormalInUEViewSpace = true;
					
					FOptiXCudaFunctionList::Get().
						CopySurfaceToCudaBuffer(
							OptiXImage->GetCudaBuffer().SurfaceObject,
							OptiXImage->GetRawImage(),
							CudaStream,
							bIsNormalInUEViewSpace);
				}
				else if (ImageType != EDenoisingImageType::OUTPUT)
				{
					OptiXImage->Flush(ECUDASurfaceCopyType::SurfaceToBuffer, CudaStream);
				}
				Denoiser->SetOptiXImage2D(ImageType, OptiXImage);
			}
			else
			{
				Denoiser->SetOptiXImage2D(ImageType, nullptr);
			}
		};

		UploadTextureToDenoiser(ColorTex, EDenoisingImageType::COLOR, TEXT("ColorTex"));
		UploadTextureToDenoiser(AlbedoTex, EDenoisingImageType::ALBEDO, TEXT("AlbedoTex"));
		UploadTextureToDenoiser(NormalTex, EDenoisingImageType::NORMAL, TEXT("NormalTex"));
		UploadTextureToDenoiser(FlowTex, EDenoisingImageType::FLOW, TEXT("FlowTex"));
		UploadTextureToDenoiser(PreviousOutputTex, EDenoisingImageType::PREVOUTPUT, TEXT("PreviousOutputTex"));
		UploadTextureToDenoiser(OutputTex, EDenoisingImageType::OUTPUT, TEXT("OutputTex"));
	}

	FOptiXDenoiseModule::Denoiser->Commit();

	auto OutputImage = FOptiXDenoiseModule::Denoiser->GetOptiXImage2D(EDenoisingImageType::OUTPUT);
	OutputImage->Flush(ECUDASurfaceCopyType::BufferToSurface, CudaStream);

	// Sync with RDG instead?
	CUDA_CHECK(FCUDAModule::CUDA().cuCtxPushCurrent(FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext()));
	CUDA_CHECK(FCUDAModule::CUDA().cuStreamSynchronize(CudaStream));
	CUDA_CHECK(FCUDAModule::CUDA().cuCtxPopCurrent(NULL));

	FOptiXDenoiseModule::OptiXImageFactory->FlushImages();
}

using namespace UE::Renderer::Private;

class FOptiXDenoiser : public IPathTracingSpatialTemporalDenoiser
{
public:
	class FHistory : public IHistory
	{
	public:
		FHistory(const TCHAR* DebugName) : DebugName(DebugName) { }

		virtual ~FHistory() { }

		const TCHAR* GetDebugName() const override { return DebugName; };

	private:
		const TCHAR* DebugName;
	};

	~FOptiXDenoiser() {}

	const TCHAR* GetDebugName() const override { return *DebugName; }

	bool NeedTextureCreateExtraFlags() const override { return true; }

	virtual FOutputs AddPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs) const
	{
		FDenoiseTextureExtParameters* DenoiseParameters = GraphBuilder.AllocParameters<FDenoiseTextureExtParameters>();
		DenoiseParameters->InputTexture = Inputs.ColorTex;
		DenoiseParameters->InputAlbedo = Inputs.AlbedoTex;
		DenoiseParameters->InputNormal = Inputs.NormalTex;
		DenoiseParameters->InputFlow = Inputs.FlowTex;
		DenoiseParameters->InputPreviousOutput = Inputs.PreviousOutputTex;
		DenoiseParameters->OutputTexture = Inputs.OutputTex;

		const int DenoisingFrameId = Inputs.DenoisingFrameId;
		const bool bForceSpatialDenoiserOnly = Inputs.bForceSpatialDenoiserOnly;

		// Need to read GPU mask outside Pass function, as the value is not refreshed inside the pass
		GraphBuilder.AddPass(RDG_EVENT_NAME("Path Tracer Denoiser Ext Plugin"), DenoiseParameters, ERDGPassFlags::Readback,
			[DenoiseParameters, DenoisingFrameId, bForceSpatialDenoiserOnly, GPUMask = View.GPUMask](FRHICommandListImmediate& RHICmdList)
		{
			Denoise(RHICmdList,
				DenoiseParameters->InputTexture->GetRHI()->GetTexture2D(),
				DenoiseParameters->InputAlbedo->GetRHI()->GetTexture2D(),
				DenoiseParameters->InputNormal->GetRHI()->GetTexture2D(),
				DenoiseParameters->OutputTexture->GetRHI()->GetTexture2D(),
				DenoiseParameters->InputFlow ? DenoiseParameters->InputFlow->GetRHI()->GetTexture2D() : nullptr,
				DenoiseParameters->InputPreviousOutput ? DenoiseParameters->InputPreviousOutput->GetRHI()->GetTexture2D() : nullptr,
				DenoisingFrameId,
				bForceSpatialDenoiserOnly,
				GPUMask);
		});

		return {};
	}

	virtual void AddMotionVectorPass(FRDGBuilder& GraphBuilder, const FSceneView& View, const FMotionVectorInputs& Inputs) const
	{
		FMotionVectorParameters* DenoiseParameters = GraphBuilder.AllocParameters<FMotionVectorParameters>();
		DenoiseParameters->InputFrameTexture = Inputs.InputFrameTex;
		DenoiseParameters->ReferenceFrameTexture = Inputs.ReferenceFrameTex;
		DenoiseParameters->OutputTexture = Inputs.OutputTex;

		const float PreExposure = Inputs.PreExposure;
		
		GraphBuilder.AddPass(RDG_EVENT_NAME("OptiX Motion Vector Pass"), DenoiseParameters, ERDGPassFlags::Readback,
			[DenoiseParameters, PreExposure, GPUMask = View.GPUMask](FRHICommandListImmediate& RHICmdList)
		{

			OpticalFlow(RHICmdList,
				DenoiseParameters->InputFrameTexture->GetRHI()->GetTexture2D(),
				DenoiseParameters->ReferenceFrameTexture->GetRHI()->GetTexture2D(),
				DenoiseParameters->OutputTexture->GetRHI()->GetTexture2D(),
				PreExposure,
				GPUMask);
		});
	}

private:
	inline static const FString DebugName = TEXT("FOptiXDenoiser");
};

void FOptiXDenoiseModule::StartupModule()
{
#if WITH_EDITOR
	UE_LOGF(LogOptiXDenoise, Log, "OptiXDenoise starting up");
#endif

	// Ensure we have the access to CUDA
	FModuleManager::LoadModuleChecked<FCUDAModule>("CUDA");

	// Register custom CUDA function list for OptiX
	// The initialization should be postponed during function call;
	FString OptiXDenoiseBaseDllPath = IPluginManager::Get().FindPlugin("OptiXDenoise")->GetBaseDir() + "/Source/ThirdParty/OptiXDenoiseBase/lib/OptiXDenoiseBase.dll";
	OptiXDenoiseBaseDLLHandle = FPlatformProcess::GetDllHandle(*OptiXDenoiseBaseDllPath);
	FOptiXCudaFunctionList::Get().RegisterFunctionInstance<FOptiXDenoiserFunctionInstance>();

	RegisterSpatialTemporalDenoiser(MakeUnique<FOptiXDenoiser>(), TEXT("OptiX"));
}

void FOptiXDenoiseModule::ShutdownModule()
{
#if WITH_EDITOR
	UE_LOGF(LogOptiXDenoise, Log, "OptiXDenoise shutting down");
#endif

	UnregisterDenoiser(TEXT("OptiX"));

	// Assure resources related to CUDA is released before the releasing of CUDA module.
	Denoiser.Reset();
	FlowEstimator.Reset();
	OptiXImageFactory.Reset();

	// Unload function list manually to ensure CUDA model is still active.
	FOptiXCudaFunctionList::Get().ShutDown();
	FPlatformProcess::FreeDllHandle(OptiXDenoiseBaseDLLHandle);
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FOptiXDenoiseModule, OptiXDenoise)
